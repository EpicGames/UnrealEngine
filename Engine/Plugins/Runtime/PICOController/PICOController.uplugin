{
	"FileVersion": 3,
	"Version": 1,
	"VersionName": "1.0",
	"FriendlyName": "PICO Controller",
	"Description": "This plugin provides support for XR_BD_controller_interaction and XR_BD_ultra_controller_interaction extensions.",
	"Category": "Virtual Reality",
	"CreatedBy": "PICO",
	"CreatedByURL": "https://www.picoxr.com",
	"DocsURL": "",
	"MarketplaceURL": "",
	"SupportURL": "",
	"EnabledByDefault": false,
	"CanContainContent": false,
	"IsBetaVersion": false,
	"Installed": false,
	"SupportedTargetPlatforms": [
		"Win64",
		"Linux",
		"Android"
	],
	"Modules": [
		{
			"Name": "PICOController",
			"Type": "Runtime",
			"LoadingPhase": "PostConfigInit",
			"PlatformAllowList": [
				"Win64",
				"Linux",
				"Android",
				"Mac"
			]
		}
	],
	"Plugins": [
		{
			"Name": "OpenXR",
			"Enabled": true
		}
	]
}