// Copyright Epic Games, Inc. All Rights Reserved.

#include "PICOController.h"
#include "InputCoreTypes.h"
#include "OpenXRCore.h"
#include "Modules/ModuleManager.h"
#include "OpenXRAssetDirectory.h"
#include "IOpenXRHMDModule.h"

#define LOCTEXT_NAMESPACE "PICOControllerModule"

// PICO Left Controller
const FKey PICO_Left_X_Click("PICO_Left_X_Click");
const FKey PICO_Left_Y_Click("PICO_Left_Y_Click");
const FKey PICO_Left_X_Touch("PICO_Left_X_Touch");
const FKey PICO_Left_Y_Touch("PICO_Left_Y_Touch");
const FKey PICO_Left_Menu_Click("PICO_Left_Menu_Click");
const FKey PICO_Left_Grip_Click("PICO_Left_Grip_Click");
const FKey PICO_Left_Grip_Axis("PICO_Left_Grip_Axis");
const FKey PICO_Left_Trigger_Click("PICO_Left_Trigger_Click");
const FKey PICO_Left_Trigger_Axis("PICO_Left_Trigger_Axis");
const FKey PICO_Left_Trigger_Touch("PICO_Left_Trigger_Touch");
const FKey PICO_Left_Thumbstick_2D("PICO_Left_Thumbstick_2D");
const FKey PICO_Left_Thumbstick_X("PICO_Left_Thumbstick_X");
const FKey PICO_Left_Thumbstick_Y("PICO_Left_Thumbstick_Y");
const FKey PICO_Left_Thumbstick_Click("PICO_Left_Thumbstick_Click");
const FKey PICO_Left_Thumbstick_Touch("PICO_Left_Thumbstick_Touch");

// PICO Right Controller
const FKey PICO_Right_A_Click("PICO_Right_A_Click");
const FKey PICO_Right_B_Click("PICO_Right_B_Click");
const FKey PICO_Right_A_Touch("PICO_Right_A_Touch");
const FKey PICO_Right_B_Touch("PICO_Right_B_Touch");
const FKey PICO_Right_Grip_Click("PICO_Right_Grip_Click");
const FKey PICO_Right_Grip_Axis("PICO_Right_Grip_Axis");
const FKey PICO_Right_Trigger_Click("PICO_Right_Trigger_Click");
const FKey PICO_Right_Trigger_Axis("PICO_Right_Trigger_Axis");
const FKey PICO_Right_Trigger_Touch("PICO_Right_Trigger_Touch");
const FKey PICO_Right_Thumbstick_2D("PICO_Right_Thumbstick_2D");
const FKey PICO_Right_Thumbstick_X("PICO_Right_Thumbstick_X");
const FKey PICO_Right_Thumbstick_Y("PICO_Right_Thumbstick_Y");
const FKey PICO_Right_Thumbstick_Click("PICO_Right_Thumbstick_Click");
const FKey PICO_Right_Thumbstick_Touch("PICO_Right_Thumbstick_Touch");

void FPICOControllerModule::StartupModule()
{
	RegisterOpenXRExtensionModularFeature();

	// Register PICO Controller
	EKeys::AddMenuCategoryDisplayInfo("PICO", LOCTEXT("PICOSubCategory", "PICO Controller"), TEXT("GraphEditor.PadEvent_16x"));

	// PICO Left Controller Keys
	EKeys::AddKey(FKeyDetails(PICO_Left_X_Click, LOCTEXT("PICO_Left_X_Click", "PICO (L) X Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Y_Click, LOCTEXT("PICO_Left_Y_Click", "PICO (L) Y Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_X_Touch, LOCTEXT("PICO_Left_X_Touch", "PICO (L) X Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Y_Touch, LOCTEXT("PICO_Left_Y_Touch", "PICO (L) Y Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Menu_Click, LOCTEXT("PICO_Left_Menu_Click", "PICO (L) Menu"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Grip_Click, LOCTEXT("PICO_Left_Grip_Click", "PICO (L) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Grip_Axis, LOCTEXT("PICO_Left_Grip_Axis", "PICO (L) Grip Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Trigger_Click, LOCTEXT("PICO_Left_Trigger_Click", "PICO (L) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Trigger_Axis, LOCTEXT("PICO_Left_Trigger_Axis", "PICO (L) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Trigger_Touch, LOCTEXT("PICO_Left_Trigger_Touch", "PICO (L) Trigger Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Thumbstick_X, LOCTEXT("PICO_Left_Thumbstick_X", "PICO (L) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Thumbstick_Y, LOCTEXT("PICO_Left_Thumbstick_Y", "PICO (L) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddPairedKey(FKeyDetails(PICO_Left_Thumbstick_2D, LOCTEXT("PICO_Left_Thumbstick_2D", "PICO (L) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "PICO"), PICO_Left_Thumbstick_X, PICO_Left_Thumbstick_Y);
	EKeys::AddKey(FKeyDetails(PICO_Left_Thumbstick_Click, LOCTEXT("PICO_Left_Thumbstick_Click", "PICO (L) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Left_Thumbstick_Touch, LOCTEXT("PICO_Left_Thumbstick_Touch", "PICO (L) Thumbstick Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));

	// PICO Right Controller Keys
	EKeys::AddKey(FKeyDetails(PICO_Right_A_Click, LOCTEXT("PICO_Right_A_Click", "PICO (R) A Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_B_Click, LOCTEXT("PICO_Right_B_Click", "PICO (R) B Press"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_A_Touch, LOCTEXT("PICO_Right_A_Touch", "PICO (R) A Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_B_Touch, LOCTEXT("PICO_Right_B_Touch", "PICO (R) B Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Grip_Click, LOCTEXT("PICO_Right_Grip_Click", "PICO (R) Grip"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Grip_Axis, LOCTEXT("PICO_Right_Grip_Axis", "PICO (R) Grip Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Trigger_Click, LOCTEXT("PICO_Right_Trigger_Click", "PICO (R) Trigger"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Trigger_Axis, LOCTEXT("PICO_Right_Trigger_Axis", "PICO (R) Trigger Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Trigger_Touch, LOCTEXT("PICO_Right_Trigger_Touch", "PICO (R) Trigger Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Thumbstick_X, LOCTEXT("PICO_Right_Thumbstick_X", "PICO (R) Thumbstick X-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Thumbstick_Y, LOCTEXT("PICO_Right_Thumbstick_Y", "PICO (R) Thumbstick Y-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis1D | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddPairedKey(FKeyDetails(PICO_Right_Thumbstick_2D, LOCTEXT("PICO_Right_Thumbstick_2D", "PICO (R) Thumbstick 2D-Axis"), FKeyDetails::GamepadKey | FKeyDetails::Axis2D | FKeyDetails::NotBlueprintBindableKey, "PICO"), PICO_Right_Thumbstick_X, PICO_Right_Thumbstick_Y);
	EKeys::AddKey(FKeyDetails(PICO_Right_Thumbstick_Click, LOCTEXT("PICO_Right_Thumbstick_Click", "PICO (R) Thumbstick Button"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
	EKeys::AddKey(FKeyDetails(PICO_Right_Thumbstick_Touch, LOCTEXT("PICO_Right_Thumbstick_Touch", "PICO (R) Thumbstick Touch"), FKeyDetails::GamepadKey | FKeyDetails::NotBlueprintBindableKey, "PICO"));
}

void FPICOControllerModule::ShutdownModule()
{
	IModularFeatures::Get().UnregisterModularFeature(GetModularFeatureName(), this);
}

bool FPICOControllerModule::GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	OutExtensions.Add("XR_BD_controller_interaction");
	return true;
}

bool FPICOControllerModule::GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions)
{
	OutExtensions.Add("XR_BD_ultra_controller_interaction");
	return true;
}

bool FPICOControllerModule::GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics)
{
	XrPath PICOInteractionProfile = XR_NULL_PATH;
	XrResult Result = XR_ERROR_VALIDATION_FAILURE;
	if (IOpenXRHMDModule::Get().IsExtensionEnabled("XR_BD_ultra_controller_interaction"))
	{
		Result = xrStringToPath(InInstance, "/interaction_profiles/bytedance/pico_ultra_controller_bd", &PICOInteractionProfile);
	}
	else
	{
		Result = xrStringToPath(InInstance, "/interaction_profiles/bytedance/pico4_controller", &PICOInteractionProfile);
	}

	if (XR_SUCCEEDED(Result) && PICOInteractionProfile != XR_NULL_PATH)
	{
		OutKeyPrefixes.Add("PICO");
		OutPaths.Add(PICOInteractionProfile);
		OutHasHaptics.Add(true);
		return true;
	}

	return false;
}

#undef LOCTEXT_NAMESPACE

IMPLEMENT_MODULE(FPICOControllerModule, PICOController)