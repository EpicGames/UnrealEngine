// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "IOpenXRExtensionPlugin.h"
#include "Modules/ModuleInterface.h"

class FPICOControllerModule :
	public IModuleInterface,
	public IOpenXRExtensionPlugin
{
public:
	FPICOControllerModule() {}

	virtual void StartupModule() override;
	virtual void ShutdownModule() override;

	virtual FString GetDisplayName() override
	{
		return FString(TEXT("PICO Controller"));
	}

	virtual bool GetRequiredExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual bool GetOptionalExtensions(TArray<const ANSICHAR*>& OutExtensions) override;
	virtual bool GetInteractionProfiles(XrInstance InInstance, TArray<FString>& OutKeyPrefixes, TArray<XrPath>& OutPaths, TArray<bool>& OutHasHaptics) override;
};
